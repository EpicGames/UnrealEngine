{
	"FileVersion" : 3,
	
	"FriendlyName" : "Reflex",
	"Version" : 1,
	"VersionName" : "1.6",
	"Description" : "NVIDIA Reflex Latency Tracking and Tick Rate Handling",
	"Category" : "Performance",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,
	
	"Modules" :
	[
		{
			"Name" : "Reflex",
			"Type" : "ClientOnly",
			"PlatformAllowList" : [ "Win64" ],
			"PlatformArchitectureDenyList" :
			[
				"Win64:arm64",
				"Win64:arm64ec"
			]
		}
	]
}
