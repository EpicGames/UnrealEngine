// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReflexBlueprint.h"

#include "ReflexLatencyMarkers.h"
#include "ReflexMaxTickRateHandler.h"	

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReflexBlueprint)

UReflexBlueprintLibrary::UReflexBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UReflexBlueprintLibrary::GetReflexAvailable()
{
	bool bIsMaxTickRateHandlerEnabled = false;

	TArray<IMaxTickRateHandlerModule*> MaxTickRateHandlerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<IMaxTickRateHandlerModule>(IMaxTickRateHandlerModule::GetModularFeatureName());
	for (IMaxTickRateHandlerModule* MaxTickRateHandler : MaxTickRateHandlerModules)
	{
		bIsMaxTickRateHandlerEnabled = bIsMaxTickRateHandlerEnabled || MaxTickRateHandler->GetAvailable();
	}

	bool bIsLatencyMarkerModuleEnabled = false;

	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		bIsLatencyMarkerModuleEnabled = bIsLatencyMarkerModuleEnabled || LatencyMarkerModule->GetAvailable(); 
	}
	
	return bIsMaxTickRateHandlerEnabled && bIsLatencyMarkerModuleEnabled;

}

void UReflexBlueprintLibrary::SetReflexMode(const EReflexMode Mode)
{
	TArray<IMaxTickRateHandlerModule*> MaxTickRateHandlerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<IMaxTickRateHandlerModule>(IMaxTickRateHandlerModule::GetModularFeatureName());
	for (IMaxTickRateHandlerModule* MaxTickRateHandler : MaxTickRateHandlerModules)
	{
		MaxTickRateHandler->SetEnabled(Mode!=EReflexMode::Disabled);
		MaxTickRateHandler->SetFlags(static_cast<uint32>(Mode));
	}

	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetEnabled(true);
	}
}

EReflexMode UReflexBlueprintLibrary::GetReflexMode()
{
	TArray<IMaxTickRateHandlerModule*> MaxTickRateHandlerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<IMaxTickRateHandlerModule>(IMaxTickRateHandlerModule::GetModularFeatureName());
	if (MaxTickRateHandlerModules.Num() > 0)
	{
		return static_cast<EReflexMode>(MaxTickRateHandlerModules[0]->GetFlags());
	}

	return EReflexMode::Disabled;
}

void UReflexBlueprintLibrary::SetFlashIndicatorEnabled(const bool bEnabled)
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetFlashIndicatorEnabled(bEnabled);
	}
}

bool UReflexBlueprintLibrary::GetFlashIndicatorEnabled()
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	if (LatencyMarkerModules.Num() > 0)
	{
		return LatencyMarkerModules[0]->GetFlashIndicatorEnabled();
	}
	
	return false;
}

float UReflexBlueprintLibrary::GetGameToRenderLatencyInMs()
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		const float Latency = LatencyMarkerModule->GetTotalLatencyInMs();
		if (Latency > 0.f)
			return Latency;
	}

	return 0.f;
}

float UReflexBlueprintLibrary::GetGameLatencyInMs()
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		const float Latency = LatencyMarkerModule->GetGameLatencyInMs();
		if (Latency > 0.f)
			return Latency;
	}

	return 0.f;
}

float UReflexBlueprintLibrary::GetRenderLatencyInMs()
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get()
		.GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		const float Latency = LatencyMarkerModule->GetRenderLatencyInMs();
		if (Latency > 0.f)
			return Latency;
	}

	return 0.f;
}

