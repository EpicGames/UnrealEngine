// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReflexLatencyMarkers.h" 
#include "ReflexModule.h"

#include "Engine/GameViewportClient.h"
#include "HAL/IConsoleManager.h"

#include "RHI.h"
#include "Engine/Console.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <D3D11.h>
#include <D3D12.h>
#include "nvapi.h"
#include "Windows/HideWindowsPlatformTypes.h"

DEFINE_LOG_CATEGORY_STATIC(LogLatencyMarkers, Log, All);

int32 DisableLatencyMarkers = 0;
static FAutoConsoleVariableRef CVarDisableLatencyMarkers(
	TEXT("t.DisableLatencyMarkers"),
	DisableLatencyMarkers,
	TEXT("Disable Latency Markers")
);

void FReflexLatencyMarkers::Initialize()
{
	bFeatureSupport = false;

	if ( (IsRHIDeviceNVIDIA()) && (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12))
	{
		bFeatureSupport = true;
		
		// Check for flash indicator sub-feature support, which still requires
		// a minimum NVIDIA driver version.
		bFlashIndicatorDriverControlled = false;

		NvU32 DriverVersion = 0;
		NvAPI_ShortString BranchString;
		NvAPI_SYS_GetDriverAndBranchVersion(&DriverVersion, BranchString);

		if (DriverVersion >= 51123)
		{
			bFlashIndicatorDriverControlled = true;
		}
	}
}

void FReflexLatencyMarkers::Tick(float DeltaTime)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status LatencyStatus = NVAPI_OK;
		NV_LATENCY_RESULT_PARAMS_V1 LatencyResults = { 0 };
		LatencyResults.version = NV_LATENCY_RESULT_PARAMS_VER1;

		LatencyStatus = NvAPI_D3D_GetLatency(FReflexModule::GetD3DDevice(), &LatencyResults);

		if (LatencyStatus == NVAPI_OK)
		{
			// frameReport[63] contains the latest completed frameReport
			const NvU64 TotalLatencyUs = LatencyResults.frameReport[63].gpuRenderEndTime - LatencyResults.frameReport[63].simStartTime;
			if (TotalLatencyUs != 0)
			{
				// frameReport results available, get latest completed frame latency data

				// A 3/4, 1/4 split gets close to a simple 10 frame moving average
				AverageTotalLatencyMs = AverageTotalLatencyMs * 0.75f + TotalLatencyUs / 1000.0f * 0.25f;

				AverageGameLatencyMs = AverageGameLatencyMs * 0.75f + (LatencyResults.frameReport[63].driverEndTime - LatencyResults.frameReport[63].simStartTime) / 1000.0f * 0.25f;
				AverageRenderLatencyMs = AverageRenderLatencyMs * 0.75f + (LatencyResults.frameReport[63].gpuRenderEndTime - LatencyResults.frameReport[63].osRenderQueueStartTime) / 1000.0f * 0.25f;

				AverageSimulationLatencyMs = AverageSimulationLatencyMs * 0.75f + (LatencyResults.frameReport[63].simEndTime - LatencyResults.frameReport[63].simStartTime) / 1000.0f * 0.25f;
				AverageRenderSubmitLatencyMs = AverageRenderSubmitLatencyMs * 0.75f + (LatencyResults.frameReport[63].renderSubmitEndTime - LatencyResults.frameReport[63].renderSubmitStartTime) / 1000.0f * 0.25f;
				AveragePresentLatencyMs = AveragePresentLatencyMs * 0.75f + (LatencyResults.frameReport[63].presentEndTime - LatencyResults.frameReport[63].presentStartTime) / 1000.0f * 0.25f;
				AverageDriverLatencyMs = AverageDriverLatencyMs * 0.75f + (LatencyResults.frameReport[63].driverEndTime - LatencyResults.frameReport[63].driverStartTime) / 1000.0f * 0.25f;
				AverageOSRenderQueueLatencyMs = AverageOSRenderQueueLatencyMs * 0.75f + (LatencyResults.frameReport[63].osRenderQueueEndTime - LatencyResults.frameReport[63].osRenderQueueStartTime) / 1000.0f * 0.25f;
				AverageGPURenderLatencyMs = AverageGPURenderLatencyMs * 0.75f + (LatencyResults.frameReport[63].gpuRenderEndTime - LatencyResults.frameReport[63].gpuRenderStartTime) / 1000.0f * 0.25f;

				RenderSubmitOffsetMs = (LatencyResults.frameReport[63].renderSubmitStartTime - LatencyResults.frameReport[63].simStartTime) / 1000.0f;
				PresentOffsetMs = (LatencyResults.frameReport[63].presentStartTime - LatencyResults.frameReport[63].simStartTime) / 1000.0f;
				DriverOffsetMs = (LatencyResults.frameReport[63].driverStartTime - LatencyResults.frameReport[63].simStartTime) / 1000.0f;
				OSRenderQueueOffsetMs = (LatencyResults.frameReport[63].osRenderQueueStartTime - LatencyResults.frameReport[63].simStartTime) / 1000.0f;
				GPURenderOffsetMs = (LatencyResults.frameReport[63].gpuRenderStartTime - LatencyResults.frameReport[63].simStartTime) / 1000.0f;

				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageTotalLatencyMs: %f", AverageTotalLatencyMs);
				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageGameLatencyMs: %f", AverageGameLatencyMs);
				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageRenderLatencyMs: %f", AverageRenderLatencyMs);

				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageSimulationLatencyMs: %f", AverageSimulationLatencyMs);
				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageRenderSubmitLatencyMs: %f", AverageRenderSubmitLatencyMs);
				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AveragePresentLatencyMs: %f", AveragePresentLatencyMs);
				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageDriverLatencyMs: %f", AverageDriverLatencyMs);
				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageOSRenderQueueLatencyMs: %f", AverageOSRenderQueueLatencyMs);
				UE_LOGF(LogLatencyMarkers, VeryVerbose, "AverageGPURenderLatencyMs: %f", AverageGPURenderLatencyMs);
			}
		}
	}
}

void FReflexLatencyMarkers::SetInputSampleLatencyMarker(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = INPUT_SAMPLE;

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetSimulationLatencyMarkerStart(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = SIMULATION_START;

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetSimulationLatencyMarkerEnd(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = SIMULATION_END;

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetRenderSubmitLatencyMarkerStart(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = RENDERSUBMIT_START;

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetRenderSubmitLatencyMarkerEnd(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = RENDERSUBMIT_END;

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetPresentLatencyMarkerStart(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = PRESENT_START;

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetPresentLatencyMarkerEnd(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = PRESENT_END;

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetFlashIndicatorLatencyMarker(uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		if (GetFlashIndicatorEnabled())
		{
			NvAPI_Status status = NVAPI_OK;
			NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
			params.version = NV_LATENCY_MARKER_PARAMS_VER1;
			params.frameID = FrameNumber;
			params.markerType = TRIGGER_FLASH;

			status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
		}
	}
}

void FReflexLatencyMarkers::SetCustomLatencyMarker(uint32 MarkerId, uint64 FrameNumber)
{
	if (DisableLatencyMarkers == 0 && bEnabled && bFeatureSupport)
	{
		NvAPI_Status status = NVAPI_OK;
		NV_LATENCY_MARKER_PARAMS_V1 params = { 0 };
		params.version = NV_LATENCY_MARKER_PARAMS_VER1;
		params.frameID = FrameNumber;
		params.markerType = NV_LATENCY_MARKER_TYPE(MarkerId);

		status = NvAPI_D3D_SetLatencyMarker(FReflexModule::GetD3DDevice(), &params);
	}
}

void FReflexLatencyMarkers::SetFlashIndicatorEnabled(bool bInEnabled)
{
	bFlashIndicatorEnabled = bInEnabled;
}

bool FReflexLatencyMarkers::GetFlashIndicatorEnabled()
{
	if (DisableLatencyMarkers == 1 || !bFeatureSupport)
	{
		return false;
	}

	return bFlashIndicatorEnabled || bFlashIndicatorDriverControlled;
}

void FReflexLatencyMarkers::SetEnabled(bool bInEnabled)
{
	if (!bInEnabled)
	{
		// Reset module back to default values in case re-enabled in the same session
		AverageTotalLatencyMs = 0.0f;
		AverageGameLatencyMs = 0.0f;
		AverageRenderLatencyMs = 0.0f;

		AverageSimulationLatencyMs = 0.0f;
		AverageRenderSubmitLatencyMs = 0.0f;
		AveragePresentLatencyMs = 0.0f;
		AverageDriverLatencyMs = 0.0f;
		AverageOSRenderQueueLatencyMs = 0.0f;
		AverageGPURenderLatencyMs = 0.0f;

		RenderSubmitOffsetMs = 0.0f;
		PresentOffsetMs = 0.0f;
		DriverOffsetMs = 0.0f;
		OSRenderQueueOffsetMs = 0.0f;
		GPURenderOffsetMs = 0.0f;

		bFlashIndicatorEnabled = false;
	}

	bEnabled = bInEnabled;
}

bool FReflexLatencyMarkers::GetEnabled()
{
	if (DisableLatencyMarkers == 1 || !bFeatureSupport)
	{
		return false;
	}

	return bEnabled;
}

bool FReflexLatencyMarkers::GetAvailable()
{
	if (DisableLatencyMarkers == 1 || !bFeatureSupport)
	{
		return false;
	}

	return true;
}

bool FReflexLatencyMarkers::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bHandled = false;
	FString ReflexLatencyMarkers;
	APlayerController* PlayerController = (InWorld ? UGameplayStatics::GetPlayerController(InWorld, 0) : NULL);
	ULocalPlayer* LocalPlayer = (PlayerController ? Cast<ULocalPlayer>(PlayerController->Player) : NULL);

	if (FParse::Value(Cmd, TEXT("ReflexLatencyMarkers="), ReflexLatencyMarkers))
	{
		if (ReflexLatencyMarkers == "0")
		{
			SetEnabled(false);

			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
			{
				LocalPlayer->ViewportClient->ViewportConsole->OutputText("Reflex Latency Markers: Off");
			}

			UE_LOGF(LogLatencyMarkers, Log, "Reflex Latency Markers: Off");
		}
		else if (ReflexLatencyMarkers == "1")
		{
			SetEnabled(true);

			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
			{
				LocalPlayer->ViewportClient->ViewportConsole->OutputText("Reflex Latency Markers: On");
			}

			UE_LOGF(LogLatencyMarkers, Log, "Reflex Latency Markers: On");
		}

		bHandled = true;
	}

	return bHandled;
}