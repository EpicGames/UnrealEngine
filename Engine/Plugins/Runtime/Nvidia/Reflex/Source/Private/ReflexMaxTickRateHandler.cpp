// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReflexMaxTickRateHandler.h" 
#include "ReflexModule.h"

#include "Engine/GameViewportClient.h"
#include "HAL/IConsoleManager.h"

#include "RHI.h"
#include "Engine/Console.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/GameplayStatics.h"
#include "Stats/ThreadIdleStats.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <D3D11.h>
#include <D3D12.h>
#include "nvapi.h"
#include "Windows/HideWindowsPlatformTypes.h"

int32 DisableCustomTickRateHandler = 0;
static FAutoConsoleVariableRef CVarDisableCustomTickRateHandler(
	TEXT("t.DisableCustomTickRateHandler"),
	DisableCustomTickRateHandler,
	TEXT("Disable Tick Rate Handler")
);

int32 DisableLatencyMarkerOptimize = 0;
static FAutoConsoleVariableRef CVarDisableLatencyMarkerOptimize(
	TEXT("t.DisableLatencyMarkerOptimize"),
	DisableLatencyMarkerOptimize,
	TEXT("Disable Latency Marker Optimize")
);

bool bEnableReflexInEditor = 0;
static FAutoConsoleVariableRef CVarEnableReflexInEditor(
	TEXT("t.EnableReflexInEditor"),
	bEnableReflexInEditor,
	TEXT("Enable Reflex in the editor")
);

DEFINE_LOG_CATEGORY_STATIC(LogMaxTickRateHandler, Log, All);

void FReflexMaxTickRateHandler::Initialize()
{
	if ((RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12) && IsRHIDeviceNVIDIA())
	{
		FString RHIName = GDynamicRHI->GetName();
		if (RHIName.StartsWith(TEXT("Vulkan")))
		{
			return;
		}

		NvU32 DriverVersion;
		NvAPI_ShortString BranchString;

		// Driver version check, 455 and above required for Reflex
		NvAPI_SYS_GetDriverAndBranchVersion(&DriverVersion, BranchString);
		if (DriverVersion >= 45500)
		{
			bProperDriverVersion = true;
		}

		// Need to verify that Reflex Low Latency mode is supported on current hardware
		NV_GET_SLEEP_STATUS_PARAMS_V1 SleepStatusParams = { 0 };
		SleepStatusParams.version = NV_GET_SLEEP_STATUS_PARAMS_VER1;

		NvAPI_Status status = NVAPI_OK;
		status = NvAPI_D3D_GetSleepStatus(FReflexModule::GetD3DDevice(), &SleepStatusParams);

		if (status == NVAPI_OK)
		{
			bFeatureSupport = true;
		}
	}
}

bool FReflexMaxTickRateHandler::HandleMaxTickRate(float DesiredMaxTickRate)
{
	if (DisableCustomTickRateHandler == 0 && bProperDriverVersion && (bEnableReflexInEditor || !GIsEditor) && IsRHIDeviceNVIDIA() && bFeatureSupport)
	{
		if (bEnabled)
		{
			const float DesiredMinimumInterval = DesiredMaxTickRate > 0 ? ((1000.0f / DesiredMaxTickRate) * 1000.0f) : 0.0f;
			if (MinimumInterval != DesiredMinimumInterval || LastCustomFlags != CustomFlags)
			{
				NvAPI_Status status = NVAPI_OK;
				NV_SET_SLEEP_MODE_PARAMS_V1 params = { 0 };
				params.version = NV_SET_SLEEP_MODE_PARAMS_VER1;
				params.bLowLatencyMode = bLowLatencyMode;
				params.bLowLatencyBoost = bBoost;
				MinimumInterval = DesiredMinimumInterval;
				params.minimumIntervalUs = MinimumInterval;
				params.bUseMarkersToOptimize = DisableLatencyMarkerOptimize ? NV_FALSE : NV_TRUE;
				status = NvAPI_D3D_SetSleepMode(FReflexModule::GetD3DDevice(), &params);

				// Need to verify that Low Latency mode change actually applied successfully
				if (bLowLatencyMode && (status != NVAPI_OK))
				{
					UE_LOGF(LogMaxTickRateHandler, Warning, "Unable to turn on low latency");
					// Clear the ULL flag
					CustomFlags = CustomFlags & ~1;
					bLowLatencyMode = false;
				}

				LastCustomFlags = CustomFlags;
				bWasEnabled = true;
			}

			NvAPI_Status StatusSleep = NVAPI_OK;
			IUnknown* const Unknown = FReflexModule::GetD3DDevice();
			{
				UE::Stats::FThreadIdleStats::FScopeIdle Scope;
				StatusSleep = NvAPI_D3D_Sleep(Unknown);
			}

			return true;
		}
		else
		{
			// When disabled, if we ever called SetSleepMode, we need to clean up after ourselves
			if (bWasEnabled)
			{
				NvAPI_Status status = NVAPI_OK;
				NV_SET_SLEEP_MODE_PARAMS_V1 params = { 0 };
				params.version = NV_SET_SLEEP_MODE_PARAMS_VER1;
				params.bLowLatencyMode = false;
				params.bLowLatencyBoost = false;
				params.minimumIntervalUs = 0;
				params.bUseMarkersToOptimize = false;
				status = NvAPI_D3D_SetSleepMode(FReflexModule::GetD3DDevice(), &params);
				UE_LOGF(LogMaxTickRateHandler, Log, "SetSleepMode clean up");

				// Reset module back to default values in case re-enabled in the same session
				MinimumInterval = -1.0f;
				LastCustomFlags = 0;
				bWasEnabled = false;
				bLowLatencyMode = true;
				bBoost = false;
			}
		}
	}

	return false;
}

void FReflexMaxTickRateHandler::SetFlags(uint32 Flags)
{
	CustomFlags = Flags;
	if ((Flags & 1) > 0)
	{
		bLowLatencyMode = true;
	}
	else
	{
		bLowLatencyMode = false;
	}
	if ((Flags & 2) > 0)
	{
		bBoost = true;
	}
	else
	{
		bBoost = false;
	}
}

uint32 FReflexMaxTickRateHandler::GetFlags()
{
	return CustomFlags;
}

void FReflexMaxTickRateHandler::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
}

bool FReflexMaxTickRateHandler::GetEnabled()
{
	if (DisableCustomTickRateHandler == 1 || !bProperDriverVersion || (GIsEditor && !bEnableReflexInEditor) || !IsRHIDeviceNVIDIA() || !bFeatureSupport)
	{
		return false;
	}

	return bEnabled;
}

bool FReflexMaxTickRateHandler::GetAvailable()
{
	if (DisableCustomTickRateHandler == 1 || !bProperDriverVersion || (GIsEditor && !bEnableReflexInEditor) || !IsRHIDeviceNVIDIA() || !bFeatureSupport)
	{
		return false;
	}

	return true;
}

bool FReflexMaxTickRateHandler::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bHandled = false;
	FString ReflexMode;
	APlayerController* PlayerController = (InWorld ? UGameplayStatics::GetPlayerController(InWorld, 0) : NULL);
	ULocalPlayer* LocalPlayer = (PlayerController ? Cast<ULocalPlayer>(PlayerController->Player) : NULL);

	if (FParse::Value(Cmd, TEXT("ReflexMode="), ReflexMode))
	{
		if (ReflexMode == "0")
		{
			SetEnabled(false);
			
			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
			{
				LocalPlayer->ViewportClient->ViewportConsole->OutputText("Reflex Low Latency mode: Off");
			}

			UE_LOGF(LogMaxTickRateHandler, Log, "Reflex Low Latency mode: Off");
		}
		else if (ReflexMode == "1")
		{
			SetEnabled(true);
			SetFlags(1);

			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
			{
				LocalPlayer->ViewportClient->ViewportConsole->OutputText("Reflex Low Latency mode: On");
			}

			UE_LOGF(LogMaxTickRateHandler, Log, "Reflex Low Latency mode: On");
		}
		else if (ReflexMode == "2")
		{
			SetEnabled(true);
			SetFlags(3);

			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
			{
				LocalPlayer->ViewportClient->ViewportConsole->OutputText("Reflex Low Latency mode: On+Boost");
			}

			UE_LOGF(LogMaxTickRateHandler, Log, "Reflex Low Latency mode: On+Boost");
		}
		
		bHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("ReflexModeToggle")))
	{
		bool bLowLatencyModeEnabled = GetEnabled();

		if (bLowLatencyModeEnabled)
		{
			SetEnabled(false);

			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
			{
				LocalPlayer->ViewportClient->ViewportConsole->OutputText("Reflex Low Latency mode: Off");
			}

			UE_LOGF(LogMaxTickRateHandler, Log, "Reflex Low Latency mode: Off");
		}
		else
		{
			SetEnabled(true);
			SetFlags(1);

			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
			{
				LocalPlayer->ViewportClient->ViewportConsole->OutputText("Reflex Low Latency mode: On");
			}

			UE_LOGF(LogMaxTickRateHandler, Log, "Reflex Low Latency mode: On");
		}

		bHandled = true;
	}

	return bHandled;
}
