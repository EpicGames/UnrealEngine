// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReflexModule.h"

#include "Modules/ModuleManager.h"

#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"

IMPLEMENT_MODULE(FReflexModule, Reflex);

void FReflexModule::StartupModule()
{
	ReflexMaxTickRateHandler = MakeUnique<FReflexMaxTickRateHandler>();
	if (ReflexMaxTickRateHandler.IsValid())
	{
		ReflexMaxTickRateHandler->Initialize();
		IModularFeatures::Get().RegisterModularFeature(ReflexMaxTickRateHandler->GetModularFeatureName(), ReflexMaxTickRateHandler.Get());
	}

	ReflexLatencyMarker = MakeUnique<FReflexLatencyMarkers>();
	if (ReflexLatencyMarker.IsValid())
	{
		ReflexLatencyMarker->Initialize();
		IModularFeatures::Get().RegisterModularFeature(ReflexLatencyMarker->GetModularFeatureName(), ReflexLatencyMarker.Get());
	}

	ReflexStatsLatencyMarker = MakeUnique<FReflexStatsLatencyMarkers>();
	if (ReflexStatsLatencyMarker.IsValid())
	{
		ReflexStatsLatencyMarker->Initialize();
		IModularFeatures::Get().RegisterModularFeature(ReflexStatsLatencyMarker->GetModularFeatureName(), ReflexStatsLatencyMarker.Get());
	}
}

void FReflexModule::ShutdownModule()
{
	if (ReflexMaxTickRateHandler.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(ReflexMaxTickRateHandler->GetModularFeatureName(), ReflexMaxTickRateHandler.Get());
	}

	if (ReflexLatencyMarker.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(ReflexLatencyMarker->GetModularFeatureName(), ReflexLatencyMarker.Get());
	}

	if (ReflexStatsLatencyMarker.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(ReflexStatsLatencyMarker->GetModularFeatureName(), ReflexStatsLatencyMarker.Get());
	}
}

IUnknown* FReflexModule::GetD3DDevice()
{
	switch (RHIGetInterfaceType())
	{
	case ERHIInterfaceType::D3D11: return GetID3D11DynamicRHI()->RHIGetDevice();
	case ERHIInterfaceType::D3D12: return GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();
	}

	return nullptr;
}
