// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "ReflexLatencyMarkers.h"
#include "ReflexMaxTickRateHandler.h"
#include "ReflexStatsLatencyMarkers.h"

class FReflexModule : public IModuleInterface
{
public:
	FReflexModule() = default;
	~FReflexModule() = default;

	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface
	
	static struct IUnknown* GetD3DDevice();

private:

	TUniquePtr<FReflexMaxTickRateHandler> ReflexMaxTickRateHandler;
	TUniquePtr<FReflexLatencyMarkers> ReflexLatencyMarker;
	TUniquePtr<FReflexStatsLatencyMarkers> ReflexStatsLatencyMarker;
};