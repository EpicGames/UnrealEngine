// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReflexStatsLatencyMarkers.h"
#include "Framework/Application/SlateApplication.h"

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <reflexstats.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

NVSTATS_DEFINE();

FReflexStatsLatencyMarkers::~FReflexStatsLatencyMarkers()
{
	if (bReflexStatsEnabled)
	{
		NVSTATS_SHUTDOWN();
	}
}

void FReflexStatsLatencyMarkers::Initialize()
{
	
}

void FReflexStatsLatencyMarkers::SetEnabled(bool bInEnabled)
{
	if (FSlateApplication::Get().IsRenderingOffScreen())
	{
		return;
	}

	FWindowsApplication* WindowsApplication = static_cast<FWindowsApplication*>(FSlateApplication::Get().GetPlatformApplication().Get());
	check(WindowsApplication);

	if (bReflexStatsEnabled)
	{
		NVSTATS_SHUTDOWN();
		WindowsApplication->RemoveMessageHandler(*this);
		bReflexStatsEnabled = false;
	}
	if (bInEnabled)
	{
		NVSTATS_INIT(0, 0);
		
		WindowsApplication->AddMessageHandler(*this);
		bReflexStatsEnabled = true;
	}
}

bool FReflexStatsLatencyMarkers::GetEnabled()
{
	return bReflexStatsEnabled;
}

bool FReflexStatsLatencyMarkers::GetAvailable()
{
	// ReflexStats is GPU agnostic, so available for all cases
	return true;
}

void FReflexStatsLatencyMarkers::SetInputSampleLatencyMarker(uint64 FrameNumber)
{
	SetCustomLatencyMarker(NVSTATS_INPUT_SAMPLE, FrameNumber);	
}

void FReflexStatsLatencyMarkers::SetSimulationLatencyMarkerStart(uint64 FrameNumber)
{
	SetCustomLatencyMarker(NVSTATS_SIMULATION_START, FrameNumber);	
}

void FReflexStatsLatencyMarkers::SetSimulationLatencyMarkerEnd(uint64 FrameNumber)
{
	SetCustomLatencyMarker(NVSTATS_SIMULATION_END, FrameNumber);	
}

void FReflexStatsLatencyMarkers::SetPresentLatencyMarkerStart(uint64 FrameNumber)
{
	SetCustomLatencyMarker(NVSTATS_PRESENT_START, FrameNumber);	
}

void FReflexStatsLatencyMarkers::SetPresentLatencyMarkerEnd(uint64 FrameNumber)
{
	SetCustomLatencyMarker(NVSTATS_PRESENT_END, FrameNumber);	
}

void FReflexStatsLatencyMarkers::SetRenderSubmitLatencyMarkerStart(uint64 FrameNumber)
{
	SetCustomLatencyMarker(NVSTATS_RENDERSUBMIT_START, FrameNumber);	
}

void FReflexStatsLatencyMarkers::SetRenderSubmitLatencyMarkerEnd(uint64 FrameNumber)
{
	SetCustomLatencyMarker(NVSTATS_RENDERSUBMIT_END, FrameNumber);
}

void FReflexStatsLatencyMarkers::SetFlashIndicatorLatencyMarker(uint64 FrameNumber)
{
	// Flash trigger should be called in any case (doesn't depend on bFlashIndicatorEnabled)
	SetCustomLatencyMarker(NVSTATS_TRIGGER_FLASH, FrameNumber);	
}

void FReflexStatsLatencyMarkers::SetCustomLatencyMarker(uint32 MarkerId, uint64 FrameNumber)
{
	if (GetEnabled())
	{
		NVSTATS_MARKER(MarkerId, FrameNumber);
	}
}

bool FReflexStatsLatencyMarkers::ProcessMessage(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam, int32& OutResult)
{
	bool bHandled = false;

	if (NVSTATS_IS_PING_MSG_ID(msg) || ((msg) == WM_KEYDOWN && ((wParam) == g_ReflexStatsVirtualKey)))
	{
		SetCustomLatencyMarker(NVSTATS_PC_LATENCY_PING, GFrameCounter);
		bHandled = true;
	}
	return bHandled;
}
