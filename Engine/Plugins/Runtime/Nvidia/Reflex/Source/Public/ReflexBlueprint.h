// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "ReflexBlueprint.generated.h"

#define UE_API REFLEX_API

UENUM(BlueprintType)
enum class EReflexMode : uint8
{
	Disabled = 0 UMETA(DisplayName="Disabled"),
	Enabled = 1 UMETA(DisplayName="Enabled"),
	EnabledPlusBoost = 3 UMETA(DisplayName="Enabled + Boost")
};


UCLASS(MinimalAPI)
class UReflexBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	UFUNCTION(BlueprintPure, Category="Reflex")
	static UE_API bool GetReflexAvailable();

	UFUNCTION(BlueprintCallable, Category="Reflex")
	static UE_API void SetReflexMode(const EReflexMode Mode);
	UFUNCTION(BlueprintPure, Category="Reflex")
	static UE_API EReflexMode GetReflexMode();

	UFUNCTION(BlueprintCallable, Category="Reflex")
	static UE_API void SetFlashIndicatorEnabled(const bool bEnabled);
	UFUNCTION(BlueprintPure, Category="Reflex")
	static UE_API bool GetFlashIndicatorEnabled();
	
	UFUNCTION(BlueprintPure, Category="Reflex")
	static UE_API float GetGameToRenderLatencyInMs();
	UFUNCTION(BlueprintPure, Category="Reflex")
	static UE_API float GetGameLatencyInMs();
	UFUNCTION(BlueprintPure, Category="Reflex")
	static UE_API float GetRenderLatencyInMs();
};

#undef UE_API
