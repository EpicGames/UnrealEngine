// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "Tickable.h"
#include "Performance/LatencyMarkerModule.h"

class FReflexLatencyMarkers : public ILatencyMarkerModule, public FTickableGameObject, public FSelfRegisteringExec
{
public:
	virtual ~FReflexLatencyMarkers() = default;

	bool bFeatureSupport = false;

	float AverageTotalLatencyMs = 0.0f;
	float AverageGameLatencyMs = 0.0f;
	float AverageRenderLatencyMs = 0.0f;

	float AverageSimulationLatencyMs = 0.0f;
	float AverageRenderSubmitLatencyMs = 0.0f;
	float AveragePresentLatencyMs = 0.0f;
	float AverageDriverLatencyMs = 0.0f;
	float AverageOSRenderQueueLatencyMs = 0.0f;
	float AverageGPURenderLatencyMs = 0.0f;

	float RenderSubmitOffsetMs = 0.0f;
	float PresentOffsetMs = 0.0f;
	float DriverOffsetMs = 0.0f;
	float OSRenderQueueOffsetMs = 0.0f;
	float GPURenderOffsetMs = 0.0f;

	bool bEnabled = false;
	bool bFlashIndicatorEnabled = false;
	bool bFlashIndicatorDriverControlled = false;

	virtual void Tick(float DeltaTime);
	virtual bool IsTickable() const { return true; }
	virtual bool IsTickableInEditor() const { return true; }
	virtual bool IsTickableWhenPaused() const { return true; }
	virtual TStatId GetStatId(void) const { RETURN_QUICK_DECLARE_CYCLE_STAT(FLatencyMarkers, STATGROUP_Tickables); }

	virtual void Initialize() override;

	virtual void SetEnabled(bool bInEnabled) override;
	virtual bool GetEnabled() override;
	virtual bool GetAvailable() override;
	virtual void SetFlashIndicatorEnabled(bool bInEnabled) override;
	virtual bool GetFlashIndicatorEnabled() override;

	virtual void SetInputSampleLatencyMarker(uint64 FrameNumber) override;
	virtual void SetSimulationLatencyMarkerStart(uint64 FrameNumber) override;
	virtual void SetSimulationLatencyMarkerEnd(uint64 FrameNumber) override;
	virtual void SetRenderSubmitLatencyMarkerStart(uint64 FrameNumber) override;
	virtual void SetRenderSubmitLatencyMarkerEnd(uint64 FrameNumber) override;
	virtual void SetPresentLatencyMarkerStart(uint64 FrameNumber) override;
	virtual void SetPresentLatencyMarkerEnd(uint64 FrameNumber) override;
	virtual void SetFlashIndicatorLatencyMarker(uint64 FrameNumber) override;
	virtual void SetCustomLatencyMarker(uint32 MarkerId, uint64 FrameNumber) override;

	virtual float GetTotalLatencyInMs() override { return AverageTotalLatencyMs; }
	virtual float GetGameLatencyInMs() override { return AverageGameLatencyMs; } // This is defined as "Game simulation start to driver submission end"
	virtual float GetRenderLatencyInMs() override { return AverageRenderLatencyMs; } // This is defined as "OS render queue start to GPU render end"

	virtual float GetSimulationLatencyInMs() override { return AverageSimulationLatencyMs; }
	virtual float GetRenderSubmitLatencyInMs() override { return AverageRenderSubmitLatencyMs; }
	virtual float GetPresentLatencyInMs() override { return AveragePresentLatencyMs; }
	virtual float GetDriverLatencyInMs() override { return AverageDriverLatencyMs; }
	virtual float GetOSRenderQueueLatencyInMs() override { return AverageOSRenderQueueLatencyMs; }
	virtual float GetGPURenderLatencyInMs() override { return AverageGPURenderLatencyMs; }

	virtual float GetRenderSubmitOffsetFromFrameStartInMs() override { return RenderSubmitOffsetMs; }
	virtual float GetPresentOffsetFromFrameStartInMs() override { return PresentOffsetMs; }
	virtual float GetDriverOffsetFromFrameStartInMs() override { return DriverOffsetMs; }
	virtual float GetOSRenderQueueOffsetFromFrameStartInMs() override { return OSRenderQueueOffsetMs; }
	virtual float GetGPURenderOffsetFromFrameStartInMs() override { return GPURenderOffsetMs; }

protected:
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
};
