// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "Performance/MaxTickRateHandlerModule.h"

class FReflexMaxTickRateHandler : public IMaxTickRateHandlerModule, public FSelfRegisteringExec
{
public:
	virtual ~FReflexMaxTickRateHandler() = default;

	virtual void Initialize() override;
	virtual void SetEnabled(bool bInEnabled) override;
	virtual bool GetEnabled() override;
	virtual bool GetAvailable() override;

	// Used to provide a generic customization interface for custom tick rate handlers
	virtual void SetFlags(uint32 Flags);
	virtual uint32 GetFlags();

	virtual bool HandleMaxTickRate(float DesiredMaxTickRate) override;

	bool bEnabled = false;
	bool bWasEnabled = false;
	bool bProperDriverVersion = false;
	bool bFeatureSupport = false;
	float MinimumInterval = -1.0f;
	bool bLowLatencyMode = true;
	bool bBoost = false;

	uint32 CustomFlags = 0;
	uint32 LastCustomFlags = 0;

protected:
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
};
