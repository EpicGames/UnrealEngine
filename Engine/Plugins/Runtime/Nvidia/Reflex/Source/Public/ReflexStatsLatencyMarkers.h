// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Performance/LatencyMarkerModule.h"
#include "Windows/WindowsApplication.h"

class FReflexStatsLatencyMarkers: public ILatencyMarkerModule, public IWindowsMessageHandler
{
public:
	virtual ~FReflexStatsLatencyMarkers();
	
	virtual void Initialize() override;
	
	virtual void SetEnabled(bool bInEnabled) override;
	virtual bool GetEnabled() override;
	virtual bool GetAvailable() override;
	
	virtual void SetFlashIndicatorEnabled(bool bInEnabled) override { bFlashIndicatorEnabled = bInEnabled; };
	virtual bool GetFlashIndicatorEnabled() override { return bFlashIndicatorEnabled; };
	
	virtual void SetInputSampleLatencyMarker(uint64 FrameNumber) override;
	virtual void SetSimulationLatencyMarkerStart(uint64 FrameNumber) override;
	virtual void SetSimulationLatencyMarkerEnd(uint64 FrameNumber) override;
	virtual void SetPresentLatencyMarkerStart(uint64 FrameNumber) override;
	virtual void SetPresentLatencyMarkerEnd(uint64 FrameNumber) override;
	virtual void SetRenderSubmitLatencyMarkerStart(uint64 FrameNumber) override;
	virtual void SetRenderSubmitLatencyMarkerEnd(uint64 FrameNumber) override;
	virtual void SetFlashIndicatorLatencyMarker(uint64 FrameNumber) override;
	virtual void SetCustomLatencyMarker(uint32 MarkerId, uint64 FrameNumber) override;

	virtual float GetTotalLatencyInMs() override { return 0.f; };
	virtual float GetGameLatencyInMs() override { return 0.f; };
	virtual float GetRenderLatencyInMs() override { return 0.f; };
	virtual float GetSimulationLatencyInMs() override { return 0.f; };
	virtual float GetRenderSubmitLatencyInMs() override { return 0.f; };
	virtual float GetPresentLatencyInMs() override { return 0.f; };
	virtual float GetDriverLatencyInMs() override { return 0.f; };
	virtual float GetOSRenderQueueLatencyInMs() override { return 0.f; };
	virtual float GetGPURenderLatencyInMs() override { return 0.f; };
	virtual float GetRenderSubmitOffsetFromFrameStartInMs() override { return 0.f; };
	virtual float GetPresentOffsetFromFrameStartInMs() override { return 0.f; };
	virtual float GetDriverOffsetFromFrameStartInMs() override { return 0.f; };
	virtual float GetOSRenderQueueOffsetFromFrameStartInMs() override { return 0.f; };
	virtual float GetGPURenderOffsetFromFrameStartInMs() override { return 0.f; };

	virtual bool ProcessMessage(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam, int32& OutResult) override;

protected:
	bool bReflexStatsEnabled = false;
	bool bFlashIndicatorEnabled = false;
};
