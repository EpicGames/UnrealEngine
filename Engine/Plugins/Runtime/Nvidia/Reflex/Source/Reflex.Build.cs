// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class Reflex : ModuleRules
	{
		public Reflex(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange([
				"Core",
				"Engine",
				"RHI",
				"CoreUObject",
				"SlateCore",
				"Slate"
			]);

			PrivateIncludePathModuleNames.AddRange([
				"D3D11RHI",
				"D3D12RHI"
			]);

			// Grab NVAPI
			AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAPI");

			// Grab ReflexStat
			PrivateIncludePaths.Add(Path.Combine(ModuleDirectory, "ThirdParty"));
		}
	}
}
